// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameMenuBuilderModule.h"
#include "Modules/ModuleManager.h"



IMPLEMENT_MODULE(FGameMenuBuilderModule, GameMenuBuilder)


