// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameMenuBuilderStyle.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "GameMenuBuilderModule.h"

TSharedPtr< FSlateStyleSet > FGameMenuBuilderStyle::SimpleStyleInstance = NULL;

void FGameMenuBuilderStyle::Initialize(const FString StyleName)
{
	if (FModuleManager::Get().IsModuleLoaded("GameMenuBuilder") == false)
	{
		FModuleManager::LoadModuleChecked<IGameMenuBuilderModule>("GameMenuBuilder");
	}
	if (!SimpleStyleInstance.IsValid())
	{
		SimpleStyleInstance = Create(StyleName);
		FSlateStyleRegistry::RegisterSlateStyle(*SimpleStyleInstance);
	}
}

void FGameMenuBuilderStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*SimpleStyleInstance);
	ensure(SimpleStyleInstance.IsUnique());
	SimpleStyleInstance.Reset();
}

FName FGameMenuBuilderStyle::GetStyleSetName()
{
	static FName StyleSetName("MenuPageStyle");
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FGameMenuBuilderStyle::Create(const FString StyleName)
{
	TSharedRef<FSlateGameResources> StyleRef = FSlateGameResources::New(GetStyleSetName(), *StyleName);
	
	const FName FontName = "Light";
	const int32 FontSize = 42;

	FSlateStyleSet& Style = StyleRef.Get();

	// Fonts still need to be specified in code for now
	Style.Set("GameMenuStyle.MenuTextStyle", FTextBlockStyle()
		.SetFont(DEFAULT_FONT(FontName, FontSize))
		.SetColorAndOpacity(FLinearColor::White)
		);
	Style.Set("GameMenuStyle.MenuHeaderTextStyle", FTextBlockStyle()
		.SetFont(DEFAULT_FONT(FontName, FontSize))
		.SetColorAndOpacity(FLinearColor::White)
		);

	return StyleRef;
}

void FGameMenuBuilderStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FGameMenuBuilderStyle::Get()
{
	return *SimpleStyleInstance;
}
