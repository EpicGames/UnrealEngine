// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameMenuPage.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWeakWidget.h"
#include "GameMenuBuilderStyle.h"
#include "GameMenuWidgetStyle.h"
#include "SGameMenuPageWidget.h"
#include "GameMenuBuilderModule.h"

FGameMenuPage::FGameMenuPage()
{
	SelectedIndex = INDEX_NONE;
}


TSharedRef<FGameMenuItem> FGameMenuPage::AddMenuItem(const FText& Text, TSharedPtr<FGameMenuPage> InSubMenu)
{
	TSharedRef<FGameMenuItem> Item = MakeShareable(new FGameMenuItem(Text));
	Item->SubMenu = InSubMenu; 
	MenuItems.Add(Item);
	return Item;
}


void FGameMenuPage::ShowRootMenu()
{
	RootMenuPageWidget->BuildAndShowMenu(this->AsShared());
	// Select the first item in the menu
	RootMenuPageWidget->SelectItem(0);
}

bool FGameMenuPage::InitialiseRootMenu(APlayerController* InPCOwner, const FString& InStyleName, UGameViewportClient* InGameViewport)
{
	if (FModuleManager::Get().IsModuleLoaded("GameMenuBuilder") == false)
	{
		FModuleManager::LoadModuleChecked<IGameMenuBuilderModule>("GameMenuBuilder");
	}	

	TArray<FString> SplitString;
	InStyleName.ParseIntoArray(SplitString, TEXT("/"), true);
	FString StylePath = InStyleName.Replace(*SplitString[SplitString.Num() - 1],TEXT(""));
	StylePath.RemoveFromEnd("/");
	FGameMenuBuilderStyle::Initialize(StylePath);

	FString InStyleName2("/Game/UI/Styles/PlatformerMenuStyle");
	bool bResult = false;
	if (InGameViewport)
	{
		SAssignNew(RootMenuPageWidget, SGameMenuPageWidget)
			.MenuStyle(&FGameMenuBuilderStyle::Get().GetWidgetStyle<FGameMenuStyle>(*InStyleName))
			.Cursor(EMouseCursor::Default)
			.PCOwner(MakeWeakObjectPtr(InPCOwner));

		InGameViewport->AddViewportWidgetContent(
			SNew(SWeakWidget)
			.PossiblyNullContent(RootMenuPageWidget));
		PCOwner = InPCOwner;
		RootMenuPageWidget->MyGameViewport = InGameViewport;
		bResult = true;
	}
	return bResult;
}

void FGameMenuPage::DestroyRootMenu()
{
	if (RootMenuPageWidget.IsValid() && RootMenuPageWidget->MyGameViewport.IsValid())
	{
		RootMenuPageWidget->ResetMenu();		
		RootMenuPageWidget->MyGameViewport->RemoveViewportWidgetContent(RootMenuPageWidget.ToSharedRef());
		RootMenuPageWidget = nullptr;
		MenuItems.Empty();
	}
}

int32 FGameMenuPage::NumItems() const
{
	return MenuItems.Num();
}

TSharedPtr<FGameMenuItem> FGameMenuPage::GetItem(int32 Index) const
{
	check(IsValidMenuEntryIndex(Index));
	return MenuItems[Index];
}

bool FGameMenuPage::IsValidMenuEntryIndex(int32 Index) const
{
	return MenuItems.IsValidIndex(Index);
}


void FGameMenuPage::LockControls(bool bLockState)
{
	if (RootMenuPageWidget.IsValid())
	{
		RootMenuPageWidget->LockControls(bLockState);
	}
}

void FGameMenuPage::HideMenu()
{
	RootMenuPageWidget->HideMenu();
}

void FGameMenuPage::RemoveAllItems()
{
	MenuItems.Empty();
}
