// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameMenuWidgetStyle.h"

FGameMenuStyle::FGameMenuStyle()
{
	LeftMarginPercent = 0.1f;
	SubMenuMarginPercent = 0.02f;
	AnimationSpeed = 0.2f;
	TextColor = FLinearColor::White;
	LayoutType = GameMenuLayoutType::Single;

	TitleHorizontalAlignment = HAlign_Center;
	TitleVerticalAlignment = VAlign_Bottom;
	PanelsVerticalAlignment = VAlign_Center;
}

FGameMenuStyle::~FGameMenuStyle()
{
}

const FName FGameMenuStyle::TypeName(TEXT("FGameMenuStyle"));

const FGameMenuStyle& FGameMenuStyle::GetDefault()
{
	static FGameMenuStyle Default;
	return Default;
}

void FGameMenuStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&MenuTopBrush);
	OutBrushes.Add(&MenuFrameBrush);
	OutBrushes.Add(&MenuBackgroundBrush);
	OutBrushes.Add(&MenuSelectBrush);
}


UGameMenuWidgetStyle::UGameMenuWidgetStyle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}
