// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameMenuItemWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Slate/SlateGameResources.h"
#include "GameMenuBuilderStyle.h"
#include "SGameMenuPageWidget.h"
#include "GameMenuPage.h"
#include "Engine/World.h"

void SGameMenuItemWidget::Construct(const FArguments& InArgs)
{
	MenuStyle = InArgs._MenuStyle;
	check(MenuStyle);

	PCOwner = InArgs._PCOwner;
	Text = InArgs._Text;
	OptionText = InArgs._OptionText;
	OnClicked = InArgs._OnClicked;
	OnArrowPressed = InArgs._OnArrowPressed;
	bIsMultichoice = InArgs._bIsMultichoice;
	bIsActiveMenuItem = false;
	LeftArrowVisible = EVisibility::Collapsed;
	RightArrowVisible = EVisibility::Collapsed;
	//if attribute is set, use its value, otherwise uses default
	InactiveTextAlpha = InArgs._InactiveTextAlpha.Get(0.5f);

	ChildSlot
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.WidthOverride(800.0f)
			.HeightOverride(68.0f)
			[
				SNew(SImage)
				.ColorAndOpacity(this,&SGameMenuItemWidget::GetButtonBgColor)
				.Image(&MenuStyle->MenuSelectBrush)
			]
		]
		+SOverlay::Slot()
		.HAlign(bIsMultichoice ? HAlign_Left : HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FGameMenuBuilderStyle::Get(), "GameMenuStyle.MenuTextStyle")
			.ColorAndOpacity(this,&SGameMenuItemWidget::GetButtonTextColor)
			.Text(Text)
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.Padding(0)
				.Visibility(this,&SGameMenuItemWidget::GetLeftArrowVisibility)
				.OnMouseButtonDown(this,&SGameMenuItemWidget::OnLeftArrowDown)
				[
					SNew(STextBlock)
					.TextStyle(FGameMenuBuilderStyle::Get(), "GameMenuStyle.MenuTextStyle")
					.ColorAndOpacity(this,&SGameMenuItemWidget::GetButtonTextColor)
					.Text(FText::FromString(TEXT("<")))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Visibility(bIsMultichoice ? EVisibility:: Visible : EVisibility::Collapsed )
				.TextStyle(FGameMenuBuilderStyle::Get(), "GameMenuStyle.MenuTextStyle")
				.ColorAndOpacity(this,&SGameMenuItemWidget::GetButtonTextColor)
				.Text(OptionText)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.Padding(0)
				.Visibility(this,&SGameMenuItemWidget::GetRightArrowVisibility)
				.OnMouseButtonDown(this,&SGameMenuItemWidget::OnRightArrowDown)
				[
					SNew(STextBlock)
					.TextStyle(FGameMenuBuilderStyle::Get(), "GameMenuStyle.MenuTextStyle")
					.ColorAndOpacity(this,&SGameMenuItemWidget::GetButtonTextColor)
					.Text(FText::FromString(TEXT(">")))
				]
			]
		]
		
	];
}

FReply SGameMenuItemWidget::OnRightArrowDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Result = FReply::Unhandled();
	const int32 MoveRight = 1;
	if (OnArrowPressed.IsBound() && bIsActiveMenuItem)
	{
		OnArrowPressed.Execute(MoveRight);
		Result = FReply::Handled();
	}
	return Result;
}

FReply SGameMenuItemWidget::OnLeftArrowDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Result = FReply::Unhandled();
	const int32 MoveLeft = -1;
	if (OnArrowPressed.IsBound() && bIsActiveMenuItem)
	{
		OnArrowPressed.Execute(MoveLeft);
		Result = FReply::Handled();
	}
	return Result;
}

EVisibility SGameMenuItemWidget::GetLeftArrowVisibility() const
{
	return LeftArrowVisible;
}

EVisibility SGameMenuItemWidget::GetRightArrowVisibility() const
{
	return RightArrowVisible;
}

FSlateColor SGameMenuItemWidget::GetButtonTextColor() const
{
	return MenuStyle->TextColor;
}

FSlateColor SGameMenuItemWidget::GetButtonBgColor() const
{
	float BgAlpha = 1.0f;
	const float MinAlpha = 0.5f;
	const float MaxAlpha = 1.0f;
	const float AnimSpeedModifier = 1.5f;
	if (PCOwner.IsValid())
	{
		float GameTime = PCOwner->GetWorld()->GetRealTimeSeconds();
		float AnimPercent = FMath::Abs(FMath::Sin(GameTime*AnimSpeedModifier));
		if (bIsActiveMenuItem)
		{
			BgAlpha = FMath::Lerp(MinAlpha, MaxAlpha, AnimPercent);
		}
		else
		{
			BgAlpha = 0.0f;
		}
	}
	return FLinearColor(1,1,1,BgAlpha);
}

FReply SGameMenuItemWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	//execute our "OnClicked" delegate, if we have one
	if(OnClicked.IsBound() == true)
	{
		return OnClicked.Execute();
	}

	return FReply::Handled();
}


FReply SGameMenuItemWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

FReply SGameMenuItemWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) 
{
	return FReply::Unhandled();
}

void SGameMenuItemWidget::SetMenuItemActive(bool bIsMenuItemActive)
{
	this->bIsActiveMenuItem = bIsMenuItemActive;
}

void SGameMenuItemWidget::SetMenuOwner(TWeakObjectPtr<class APlayerController> InPCOwner)
{
	PCOwner = InPCOwner;
}

void SGameMenuItemWidget::SetMenuStyle(const FGameMenuStyle* InMenuStyle)
{
	MenuStyle = InMenuStyle;
}

void SGameMenuItemWidget::SetClickedDelegate(FOnClicked InOnClicked)
{
	OnClicked = InOnClicked;
}

void SGameMenuItemWidget::SetArrowPressedDelegate(FOnArrowPressed InOnArrowPressed)
{
	OnArrowPressed = InOnArrowPressed;
}
