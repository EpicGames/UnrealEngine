// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameMenuPageWidget.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "GameMenuBuilderStyle.h"
#include "SGameMenuItemWidget.h"
#include "GameMenuPage.h"
#include "Engine/Console.h"
#include "Widgets/Layout/SDPIScaler.h"


FMenuPanel::FMenuPanel()
{
	CurrentState = EPanelState::Closed;

	// Setup a default animation curve
	AnimationSequence = FCurveSequence();
	AnimationHandle = AnimationSequence.AddCurve(0, 0.2f, ECurveEaseFunction::QuadInOut);
}

void FMenuPanel::SetStyle(const FGameMenuStyle* InStyle)
{
	AnimationSequence = FCurveSequence();	
	AnimationHandle = AnimationSequence.AddCurve(0, InStyle->AnimationSpeed, ECurveEaseFunction::QuadInOut);
}

void FMenuPanel::Tick(float Delta)
{
	if (AnimationSequence.IsPlaying() == false)
	{
		EPanelState::Type OldState = CurrentState;
		switch (CurrentState)
		{
		case EPanelState::Opening:
			CurrentState = EPanelState::Open;
			break;
		case EPanelState::Closing:
			CurrentState = EPanelState::Closed;
			break;
		}
		if (OldState != CurrentState)
		{
			OnStateChanged.ExecuteIfBound(CurrentState == EPanelState::Open);
		}
	}
}

void FMenuPanel::ClosePanel(TSharedRef<SWidget> OwnerWidget)
{
	if ((CurrentState != EPanelState::Closed) && (CurrentState != EPanelState::Closing))
	{
		if ((AnimationSequence.IsPlaying() == true) && (CurrentState == EPanelState::Opening))
		{
			AnimationSequence.Reverse();
		}
		else
		{
			AnimationSequence.PlayReverse(OwnerWidget);
		}
		CurrentState = EPanelState::Closing;
	}
	else
	{
		if (AnimationSequence.IsPlaying() == false)
		{
			AnimationSequence.JumpToStart();
		}
		if (CurrentState != EPanelState::Closing)
		{
			OnStateChanged.ExecuteIfBound(CurrentState == EPanelState::Open);
		}
	}
}

void FMenuPanel::OpenPanel(TSharedRef<SWidget> OwnerWidget)
{
	if ((CurrentState != EPanelState::Open) && (CurrentState != EPanelState::Opening) )
	{
		if ((AnimationSequence.IsPlaying() == true) && (CurrentState == EPanelState::Closing))
		{
			AnimationSequence.Reverse();
		}
		else
		{
			AnimationSequence.Play(OwnerWidget);
		}
		CurrentState = EPanelState::Opening;
	}
	else
	{
		if (AnimationSequence.IsPlaying() == false)
		{
			AnimationSequence.JumpToEnd();
		}
		if (CurrentState != EPanelState::Opening)
		{
			OnStateChanged.ExecuteIfBound(CurrentState == EPanelState::Open);
		}
	}
}

void FMenuPanel::ForcePanelOpen()
{
	AnimationSequence.JumpToEnd();
	if (CurrentState != EPanelState::Open)
	{
		CurrentState = EPanelState::Open;
		OnStateChanged.ExecuteIfBound(CurrentState == EPanelState::Open); //-V547
	}
}

void FMenuPanel::ForcePanelClosed()
{
	AnimationSequence.JumpToStart();
	if (CurrentState != EPanelState::Closed)
	{
		CurrentState = EPanelState::Closed;
		OnStateChanged.ExecuteIfBound(CurrentState == EPanelState::Open); //-V547
	}
}

void SGameMenuPageWidget::Construct(const FArguments& InArgs)
{
	MenuStyle = InArgs._MenuStyle;
	check(MenuStyle);
	MainMenuPanel.SetStyle(MenuStyle);
	SubMenuPanel.SetStyle(MenuStyle);
	
	EHorizontalAlignment MainAlignmentH;
	if (MenuStyle->LayoutType == GameMenuLayoutType::Single)
	{
		MainAlignmentH = EHorizontalAlignment::HAlign_Center;
	}
	else
	{
		MainAlignmentH = EHorizontalAlignment::HAlign_Left; 
	}
		
	SetupAnimations();
	TitleWidgetAnimation.JumpToEnd();
	
	bControlsLocked = false;
	bConsoleVisible = false;
	bMenuHiding = false;
	bMenuHidden = true;
	SelectedIndex = INDEX_NONE;
	

	PCOwner = InArgs._PCOwner;
	bGameMenu = InArgs._GameMenu;
	UIScale.Bind(this, &SGameMenuPageWidget::GetUIScale);
	//ControllerHideMenuKey = EKeys::Gamepad_Special_Right;
	SetVisibility(MakeAttributeSP(this, &SGameMenuPageWidget::GetSlateVisibility));
		
	// Create the title widget
	TSharedRef<SHorizontalBox> TitleBoxWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetMenuTitleOffset))		
	[
		SNew(SBorder)
		.BorderImage(&MenuStyle->MenuTopBrush)
		.Padding(MenuStyle->TitleBorderMargin)
		.Visibility(this, &SGameMenuPageWidget::GetMenuTitleVisibility)
		[
			SNew(STextBlock)
			.TextStyle(FGameMenuBuilderStyle::Get(), "GameMenuStyle.MenuHeaderTextStyle")
			.ColorAndOpacity(this, &SGameMenuPageWidget::GetTitleColor)
			.Text(this, &SGameMenuPageWidget::GetMenuTitle)
		]
	];

	// Create the widget that houses the 2 menu panels
	TSharedRef< SHorizontalBox >	PanelBoxes = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.HAlign(MainAlignmentH)
	.VAlign(MenuStyle->PanelsVerticalAlignment)
	[
		// The main menu
 		SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetMainMenuOffset))
		[
			SNew(SImage)
			.ColorAndOpacity(this, &SGameMenuPageWidget::GetPanelsBackgroundColor)
			.Image(&MenuStyle->MenuBackgroundBrush)
		]	
		+ SOverlay::Slot()
		.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetMainMenuOffset))
		[
			SNew(SBorder)
			.ColorAndOpacity(this, &SGameMenuPageWidget::GetPanelsColor)
			.BorderImage(&MenuStyle->MenuFrameBrush)
			.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetMenuItemPadding)/*MenuStyle->ItemBorderMargin*/)
			[
				SAssignNew(MainPanel, SVerticalBox)
			]
		]
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(MainAlignmentH)
	.VAlign(MenuStyle->PanelsVerticalAlignment)
	[
		// The sub menu
		SNew(SOverlay)
		.Visibility(this, &SGameMenuPageWidget::GetSubMenuVisiblity)
		+ SOverlay::Slot()
		.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetSubMenuOffset))
		[
			SNew(SImage)
			.ColorAndOpacity(this, &SGameMenuPageWidget::GetPanelsBackgroundColor)
			.Image(&MenuStyle->MenuBackgroundBrush)
		]
		+ SOverlay::Slot()
			.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetSubMenuOffset))
		[
			SNew(SBorder)
			.ColorAndOpacity(this, &SGameMenuPageWidget::GetPanelsColor)
			.BorderImage(&MenuStyle->MenuFrameBrush)		
			.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetSubMenuItemPadding)/*MenuStyle->ItemBorderMargin*/)
			[
				SAssignNew(SubPanel, SVerticalBox)
			]
		]
	];
	
	// Create the main widget
	ChildSlot
	[
		SNew(SDPIScaler)
		.DPIScale(UIScale)
		[
			SNew(SOverlay)
			
			+ SOverlay::Slot()
 			.HAlign(HAlign_Fill)
 			.VAlign(VAlign_Fill)
			.Padding(TAttribute<FMargin>(this, &SGameMenuPageWidget::GetMenuOffset))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(MenuStyle->TitleHorizontalAlignment)
				.VAlign(MenuStyle->TitleVerticalAlignment)				
				[
					TitleBoxWidget
				]
				
				+ SVerticalBox::Slot()
				.HAlign(MainAlignmentH)
				.VAlign(MenuStyle->PanelsVerticalAlignment)
				.Padding(FMargin(58.0f))
				[						
					PanelBoxes
				]			
			]				//SOverlay
		]					//SDPIScaler
	];
	
}

EVisibility SGameMenuPageWidget::GetSubMenuVisiblity() const
{
	EVisibility MenuVisibility = EVisibility::Collapsed;
	if ( (MenuStyle->LayoutType != GameMenuLayoutType::Single) && ( bMenuHiding == false ) )
	{
		MenuVisibility = NextMenu.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return MenuVisibility;
}

EVisibility SGameMenuPageWidget::GetSlateVisibility() const
{
	EVisibility Visible = EVisibility::Visible;
	if (bMenuHidden  == true)
	{
		Visible = EVisibility::Collapsed;
	}
	else if ( (bConsoleVisible == true) || (bMenuHiding == true ) )
	{
		Visible = EVisibility::HitTestInvisible;
	}
	return Visible;
}

EVisibility SGameMenuPageWidget::GetMenuTitleVisibility() const
{
	return (CurrentMenuTitle.IsEmpty() || bMenuHidden == true) ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SGameMenuPageWidget::GetMenuTitle() const 
{
	return CurrentMenuTitle;
}

void SGameMenuPageWidget::SetCurrentMenu(TSharedPtr< class FGameMenuPage > InMenu)
{
	if (InMenu.IsValid())
	{
		if (CurrentMenu.IsValid() && CurrentMenu->RootMenuPageWidget.IsValid())
		{
			InMenu->RootMenuPageWidget = CurrentMenu->RootMenuPageWidget;
		}
		CurrentMenu = InMenu;		
	}
}

bool SGameMenuPageWidget::SelectItem(int32 InSelection)
{
	bool bResult = false;
	if (InSelection != SelectedIndex)
	{
		SelectionChanged(0);
		bResult = true;
	}
	return bResult;
}

void SGameMenuPageWidget::BuildAndShowMenu(TSharedPtr< class FGameMenuPage > InMenu)
{
	SetCurrentMenu(InMenu);

	bMenuHiding = false;
	bMenuHidden = false;
		
	MainMenuPanel.OnStateChanged.Unbind();
	SubMenuPanel.OnStateChanged.Unbind();
	OpenMainPanel(CurrentMenu);
	MainMenuPanel.OnStateChanged.BindSP(this, &SGameMenuPageWidget::OnMainPanelStateChange);
	SubMenuPanel.OnStateChanged.BindSP(this, &SGameMenuPageWidget::OnSubPanelStateChange);	
	FSlateApplication::Get().PlaySound(MenuStyle->MenuEnterSound);
}

void SGameMenuPageWidget::HideMenu() 
{
	if (!bMenuHiding)
	{
		MainMenuPanel.ClosePanel(this->AsShared());
		SubMenuPanel.ClosePanel(this->AsShared());
		PendingMainMenu.Reset();
		PendingSubMenu.Reset();
		bMenuHiding = true;
	}
}

void SGameMenuPageWidget::SetupAnimations()
{
	//Setup a curve
	const float StartDelay = 0.0f;
	const float SecondDelay = bGameMenu ? 0.f : 0.3f;
	
	if (!bGameMenu)
	{
		//when in main menu, only animate from left or from right side of the screen
		MainAnimNumber = FMath::RandRange(0,1);
	} 
	else
	{
		//in game menu can use animations from top or from bottom too
		MainAnimNumber = FMath::RandRange(0,3);
	}
	
	MenuWidgetAnimation = FCurveSequence();	
		
	// Color fading
	ColorFadeCurve = MenuWidgetAnimation.AddCurve(StartDelay + SecondDelay, MenuStyle->AnimationSpeed, ECurveEaseFunction::QuadInOut);
	
	// sliding animation
	MenuAnimationCurve = MenuWidgetAnimation.AddCurve(StartDelay + SecondDelay, MenuStyle->AnimationSpeed, ECurveEaseFunction::QuadInOut);

	// Animation for the title
	TitleWidgetAnimation = FCurveSequence();
	TitleWidgetCurve = TitleWidgetAnimation.AddCurve(StartDelay, MenuStyle->AnimationSpeed, ECurveEaseFunction::QuadInOut);
}

void SGameMenuPageWidget::BuildPanelButtons(TSharedPtr< class FGameMenuPage > InPanel, TSharedPtr<SVerticalBox> InBox, int32 InPreviousIndex)
{
	InBox->ClearChildren();
	
	if (InPanel.IsValid() == true)
	{
		for (int32 i = 0; i < InPanel->NumItems(); ++i)
		{
			TSharedPtr<SGameMenuItemWidget> TmpWidget;
			TSharedPtr<FGameMenuItem> EachItem = InPanel->GetItem(i);
			if (EachItem.Get()->MenuItemType == EGameMenuItemType::Standard)
			{
				TmpWidget = SAssignNew(EachItem->Widget, SGameMenuItemWidget)
					.MenuStyle(MenuStyle)
					.PCOwner(PCOwner)
					.OnClicked(this, &SGameMenuPageWidget::MouseButtonClicked, i)
					.Text(EachItem.Get()->Text)
					.bIsMultichoice(false);
			}
			else if (EachItem->MenuItemType == EGameMenuItemType::MultiChoice)
			{
				TmpWidget = SAssignNew(EachItem->Widget, SGameMenuItemWidget)
					.MenuStyle(MenuStyle)
					.PCOwner(PCOwner)
					.OnClicked(this, &SGameMenuPageWidget::MouseButtonClicked, i)
					.Text(EachItem.Get()->Text)
					.bIsMultichoice(true)
					.OnArrowPressed(this, &SGameMenuPageWidget::ChangeOption)
					.OptionText(this, &SGameMenuPageWidget::GetOptionText, EachItem);
				UpdateArrows(EachItem);
			}
			else if (EachItem.Get()->MenuItemType == EGameMenuItemType::CustomWidget)
			{
				TmpWidget = EachItem.Get()->CustomWidget;
				TmpWidget->SetMenuOwner(PCOwner);
				TmpWidget->SetMenuStyle(MenuStyle);
			}
			InBox->AddSlot().HAlign(HAlign_Left).AutoHeight()
				[
					TmpWidget.ToSharedRef()
				];
		}
		
		if ( InPreviousIndex != INDEX_NONE )
		{
			SelectedIndex = InPreviousIndex;
			TSharedPtr<FGameMenuItem> FirstMenuItem = InPanel->IsValidMenuEntryIndex(SelectedIndex) ? InPanel->GetItem(InPreviousIndex) : NULL;
			if (FirstMenuItem.IsValid() == true )
			{
				if (FirstMenuItem->MenuItemType != EGameMenuItemType::CustomWidget)
				{
					FirstMenuItem->Widget->SetMenuItemActive(true);
					FSlateApplication::Get().SetKeyboardFocus(SharedThis(this));
				}
				// If the selection has a we need to mark for pending so it will open (in side by side layout)
				PendingSubMenu = FirstMenuItem->SubMenu;
			}			
		}
	}

}

FText SGameMenuPageWidget::GetOptionText(TSharedPtr<FGameMenuItem> InMenuItem) const
{
	return InMenuItem->MultiChoice[InMenuItem->SelectedMultiChoice];
}

void SGameMenuPageWidget::UpdateArrows(TSharedPtr<FGameMenuItem> InMenuItem)
{
	const int32 MinIndex = InMenuItem->MinMultiChoiceIndex > -1 ? InMenuItem->MinMultiChoiceIndex : 0;
	const int32 MaxIndex = InMenuItem->MaxMultiChoiceIndex > -1 ? InMenuItem->MaxMultiChoiceIndex : InMenuItem->MultiChoice.Num()-1;
	const int32 CurIndex = InMenuItem->SelectedMultiChoice;
	if (CurIndex > MinIndex)
	{
		InMenuItem->Widget->LeftArrowVisible = EVisibility::Visible;
	} 
	else
	{
		InMenuItem->Widget->LeftArrowVisible = EVisibility::Collapsed;
	}
	if (CurIndex < MaxIndex)
	{
		InMenuItem->Widget->RightArrowVisible = EVisibility::Visible;
	}
	else
	{
		InMenuItem->Widget->RightArrowVisible = EVisibility::Collapsed;
	}
}

void SGameMenuPageWidget::EnterSubMenu(TSharedPtr<class FGameMenuPage> InSubMenu)
{
	if (InSubMenu.IsValid())
	{
		MenuHistory.Push(CurrentMenu);
		CurrentMenu->SelectedIndex = SelectedIndex;
		MainMenuPanel.ClosePanel(this->AsShared());
		PendingMainMenu = InSubMenu;
		FSlateApplication::Get().PlaySound(MenuStyle->MenuEnterSound);
	}
}

void SGameMenuPageWidget::MenuGoBack(bool bIsCancel)
{
	if (MenuHistory.Num() > 0)
	{		
		TSharedPtr< FGameMenuPage > MenuInfo = MenuHistory.Pop();
		if (MainMenuPanel.CurrentState == EPanelState::Closing)
		{
			MainMenuPanel.OpenPanel(this->AsShared());
			PendingMainMenu.Reset();
			PendingSubMenu.Reset();
		}
		else
		{
			CurrentMenu->SelectedIndex = SelectedIndex;
			if (MenuStyle->LayoutType == GameMenuLayoutType::Single)
			{
				// single menu layout - close this panel and replace with prev menu
				PendingMainMenu = MenuInfo;
				MainMenuPanel.ClosePanel(this->AsShared());
			}
			else
			{
				// side by side layout means the current menu becomes sub menu - and the main is the one we are going back too				
				BuildPanelButtons(CurrentMenu, SubPanel, INDEX_NONE);
				SubMenuPanel.ForcePanelOpen();
				MainMenuPanel.ForcePanelClosed();
				OpenMainPanel(MenuInfo);
			}
		}
		FSlateApplication::Get().PlaySound(MenuStyle->MenuBackSound);
		if (bIsCancel == true)
		{
			CurrentMenu->MenuGoBackCancel();
		}
		else
		{
			CurrentMenu->MenuGoBack();
		}		
	}
	else if (bGameMenu) // only when it's in-game menu variant
	{
		if (MenuStyle->MenuBackSound.GetResourceObject() != nullptr)
		{
			FSlateApplication::Get().PlaySound(MenuStyle->MenuBackSound);
		}
		// We are sort of going back here too
		if (bIsCancel == true)
		{
			CurrentMenu->MenuGoBackCancel();
		}
		else
		{
			CurrentMenu->MenuGoBack();
		}
		OnToggleMenu.ExecuteIfBound();
	}
}

void SGameMenuPageWidget::ConfirmMenuItem()
{
	if (CurrentMenu.IsValid()==true)
	{
		TSharedPtr<FGameMenuItem> CurrentMenuItem = CurrentMenu->GetItem(SelectedIndex);
		if ( CurrentMenuItem.IsValid() == true )
		{
			bool bItemConfirmed = false;
			if( CurrentMenuItem->ConfirmPressed() == true)
 			{
				bItemConfirmed = true;
 			}
			
			// We dont want to play 2 menu sounds here
			if (CurrentMenuItem->SubMenu.IsValid())
			{
				EnterSubMenu(CurrentMenuItem->SubMenu); 
				FSlateApplication::Get().PlaySound(MenuStyle->MenuEnterSound);				
			}
			else if (bItemConfirmed == true)
			{
				FSlateApplication::Get().PlaySound(MenuStyle->MenuItemChosenSound);
			}
		}
	}
}

void SGameMenuPageWidget::OpenMainPanel(TSharedPtr< class FGameMenuPage > InMenu )
{
	int32 PreviousIndex = 0;
	MainPanel->ClearChildren();
	
	SetCurrentMenu(InMenu);

	PreviousIndex = CurrentMenu->SelectedIndex;
	// If we have not got a previous index select the first item as that now
	if (PreviousIndex == INDEX_NONE)
	{
		PreviousIndex = 0;
	}
	if( ( PreviousIndex != INDEX_NONE ) && (CurrentMenu->NumItems() > PreviousIndex) )
	{
		NextMenu = CurrentMenu->GetItem(PreviousIndex)->SubMenu;
	}
	BuildPanelButtons(CurrentMenu, MainPanel, PreviousIndex);
	CurrentMenuTitle = InMenu->MenuTitle;

	MainMenuPanel.OpenPanel(this->AsShared());
	CurrentMenu->MenuOpening();

	OpenPendingSubMenu();	
}

void SGameMenuPageWidget::OnMainPanelStateChange(bool bWasOpened)
{
	if (bWasOpened == false)
	{
		if (PendingMainMenu.IsValid())
		{
			OpenMainPanel(PendingMainMenu);
			if (MenuStyle->LayoutType != GameMenuLayoutType::Single)
			{
				MainMenuPanel.ForcePanelOpen();
				SubMenuPanel.ForcePanelClosed();				
			}
			PendingMainMenu.Reset();
		}
		if ((PendingSubMenu.IsValid() == false) && (bMenuHiding == true))
		{
			bMenuHiding = false;
			bMenuHidden = true;
			//Send event, if we have one bound
			if (CurrentMenu.IsValid())
			{
				CurrentMenu->MenuHidden();
			}
		}
		OpenPendingSubMenu();
	}
	else
	{
		OpenPendingSubMenu();
	}
}

void SGameMenuPageWidget::OnSubPanelStateChange(bool bWasOpened)
{
	if (bWasOpened == false)
	{
		OpenPendingSubMenu();
	}
}

void SGameMenuPageWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	//ugly code seeing if the console is open
	UConsole* ViewportConsole = (GEngine && GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
	if ( ViewportConsole != nullptr && ( ViewportConsole->ConsoleState == "Typing" || ViewportConsole->ConsoleState == "Open" ) )
	{
		if (!bConsoleVisible)
		{
			bConsoleVisible = true;
			FSlateApplication::Get().SetAllUserFocusToGameViewport();
		}
	}
	else
	{
		if (bConsoleVisible)
		{
			bConsoleVisible = false;
			FSlateApplication::Get().SetKeyboardFocus(SharedThis(this));
		}
	}

	if (GEngine && GEngine->GameViewport )
	{
		if ( GEngine->GameViewport->Viewport != nullptr )
		{
			ScreenRes = GEngine->GameViewport->Viewport->GetSizeXY();
		}
		else
		{
			FVector2D ViewSize;
			GEngine->GameViewport->GetViewportSize(ViewSize);
			ScreenRes.X = ViewSize.X;
			ScreenRes.Y = ViewSize.Y;
		}
	}
	

	if (MenuWidgetAnimation.IsAtStart() && !bMenuHiding)
	{
		//Start the menu widget animation, set keyboard focus
		FadeIn();
	}

	if (MenuWidgetAnimation.IsAtEnd())
	{
		MainMenuPanel.Tick(InDeltaTime);
		SubMenuPanel.Tick(InDeltaTime);		
	}	
}

float SGameMenuPageWidget::GetUIScale() const
{
	return ScreenRes.X / 2048.0f;
}

FMargin SGameMenuPageWidget::GetMenuOffset() const
{
	FMargin Result;
	if ( CurrentMenu.IsValid() == true )
	{
		const float WidgetWidth = MainPanel->GetDesiredSize().X;
		const float WidgetHeight = MainPanel->GetDesiredSize().Y;
		const float CachedScale = UIScale.Get();
		const float VirtualScreenWidth = ScreenRes.X / CachedScale;
		const float VirtualScreenHeight = ScreenRes.Y / CachedScale;		
		const float AnimProgress = MenuAnimationCurve.GetLerp();
		const float OffsetX = 0.0f;

		switch (MainAnimNumber)
		{
		case 0:
			Result = FMargin(OffsetX + VirtualScreenWidth - AnimProgress * VirtualScreenWidth, 0, 0, 0);
			break;
		case 1:
			Result = FMargin(OffsetX - VirtualScreenWidth + AnimProgress * VirtualScreenWidth, 0, 0, 0);
			break;
		case 2:
			Result = FMargin(OffsetX, VirtualScreenHeight - AnimProgress * VirtualScreenHeight, 0, 0);
			break;
		case 3:
			Result = FMargin(OffsetX, -VirtualScreenHeight + AnimProgress * VirtualScreenHeight, 0, 0);
			break;
		}
	}
	
	return Result;
}

FMargin SGameMenuPageWidget::GetMenuTitleOffset() const
{
 	const float WidgetWidth = MainPanel->GetDesiredSize().X;
 	const float BoxSizeX = MainPanel->GetDesiredSize().X;
	const float RightMargin = -WidgetWidth + TitleWidgetAnimation.GetLerp() * WidgetWidth;
	const float OutlineWidth = 2.0f;
	return FMargin(OutlineWidth, OutlineWidth, RightMargin, OutlineWidth);
}

FMargin SGameMenuPageWidget::GetMenuItemPadding() const
{
	const float WidgetWidth = MainPanel->GetDesiredSize().X;
	float Lerp = MainMenuPanel.AnimationHandle.GetLerp();
	float WholeSize = -((WidgetWidth - (WidgetWidth*Lerp)) / 2);
	float RealMargin = WholeSize;
	return FMargin(RealMargin + (MenuStyle->ItemBorderMargin.Left*Lerp), MenuStyle->ItemBorderMargin.Top, RealMargin + (MenuStyle->ItemBorderMargin.Right*Lerp), MenuStyle->ItemBorderMargin.Bottom);
}

FMargin SGameMenuPageWidget::GetSubMenuItemPadding() const
{
	const float WidgetWidth = MainPanel->GetDesiredSize().X;
	float Lerp = SubMenuPanel.AnimationHandle.GetLerp();
	float WholeSize = -((WidgetWidth - (WidgetWidth*Lerp)) / 2);
	float RealMargin = WholeSize;
	return FMargin(RealMargin + (MenuStyle->ItemBorderMargin.Left*Lerp), MenuStyle->ItemBorderMargin.Top, RealMargin + (MenuStyle->ItemBorderMargin.Right*Lerp), MenuStyle->ItemBorderMargin.Bottom);
}

FMargin SGameMenuPageWidget::GetMainMenuOffset() const
{
	const float WidgetWidth = MainPanel->GetDesiredSize().X;
	const float LeftBoxSizeX = MainPanel->GetDesiredSize().X;
	float Lerp = MainMenuPanel.AnimationHandle.GetLerp();
	float LeftMargin = 0.0f;
	float RightMargin = 0.0;
	if (MenuStyle->LayoutType == GameMenuLayoutType::Single)
	{
		float Size = WidgetWidth * Lerp;
		RightMargin = LeftMargin = (WidgetWidth - Size) / 2.0f;
	}
	else
	{
		LeftMargin = WidgetWidth * MenuStyle->LeftMarginPercent;
		RightMargin = -LeftBoxSizeX + Lerp * LeftBoxSizeX;
	}
	return FMargin(LeftMargin, 0, RightMargin, 0);
}


FMargin SGameMenuPageWidget::GetSubMenuOffset() const
{
	if (SubPanel.IsValid())
	{
		const float WidgetWidth = SubPanel->GetDesiredSize().X;
		const float RightBoxSizeX = WidgetWidth;
		const float LeftMargin = WidgetWidth * MenuStyle->SubMenuMarginPercent;
		float Lerp = SubMenuPanel.AnimationHandle.GetLerp();
		const float RightMargin = -RightBoxSizeX + Lerp * RightBoxSizeX;
		return FMargin(LeftMargin, 0, RightMargin, 0);
	}
	else
	{
		return FMargin();
	}
}

FSlateColor SGameMenuPageWidget::GetTitleColor() const
{
	FSlateColor VisibleColor = CurrentMenuTitle.IsEmpty() ? FLinearColor::Transparent : FLinearColor::White;
	if (MenuStyle != NULL)
	{
		VisibleColor = GetTextColor();
	}
	return 	VisibleColor;
}

FLinearColor SGameMenuPageWidget::GetPanelsColor() const
{
	return FMath::Lerp(FLinearColor(1,1,1,0), FLinearColor(1,1,1,1), ColorFadeCurve.GetLerp());
}

FSlateColor SGameMenuPageWidget::GetPanelsBackgroundColor() const
{
	return FMath::Lerp(FLinearColor(1, 1, 1, 0), FLinearColor(1, 1, 1, 1), ColorFadeCurve.GetLerp());
}

FSlateColor SGameMenuPageWidget::GetTextColor() const
{
	return MenuStyle->TextColor;
}

FReply SGameMenuPageWidget::MouseButtonClicked(int32 SelectionIndex)
{
	FReply Reply = FReply::Unhandled();
	if (CurrentMenu.IsValid() == true)
	{
		if (bControlsLocked)
		{
			Reply = FReply::Handled();
		}
		else 
		{
			if (SelectedIndex != SelectionIndex)
			{
				SelectionChanged(SelectionIndex);
			}
			ConfirmMenuItem();
		}
	}
	return Reply;
}

FReply SGameMenuPageWidget::SelectionChanged(int32 SelectionIndex)
{	
	if (CurrentMenu.IsValid() == false)
	{
		return FReply::Unhandled();
	}
	if (bControlsLocked)
	{
		return FReply::Handled();
	}
	if (SelectedIndex != SelectionIndex)
	{
		TSharedPtr<FGameMenuItem> CurrentMenuItem;
		if (SelectedIndex != INDEX_NONE)
		{
			CurrentMenuItem = CurrentMenu->GetItem(SelectedIndex);

			if (CurrentMenuItem.IsValid())
			{
				TSharedPtr<SGameMenuItemWidget> MenuWidgetItem = CurrentMenuItem->Widget;
				MenuWidgetItem->SetMenuItemActive(false);
			}
		}
		SelectedIndex = SelectionIndex;
		TSharedPtr<FGameMenuItem> NewMenuItem = CurrentMenu->GetItem(SelectedIndex);
		NewMenuItem->Widget->SetMenuItemActive(true);
		FSlateApplication::Get().PlaySound(MenuStyle->MenuItemChangeSound);

		OnSelectionChange.ExecuteIfBound(CurrentMenuItem, NewMenuItem);
				
		PendingSubMenu = NewMenuItem->SubMenu;
		if ((PendingSubMenu.IsValid() == false) && (MenuStyle->LayoutType == GameMenuLayoutType::SideBySide))
		{			
			PendingSubMenu.Reset();
		}
		if ((SubMenuPanel.CurrentState == EPanelState::Open) || (SubMenuPanel.CurrentState == EPanelState::Opening))
		{
			SubMenuPanel.ClosePanel(this->AsShared());
		}
		else
		{
			OpenPendingSubMenu();
		}
	}
	else if (SelectedIndex == SelectionIndex)
	{
		ConfirmMenuItem();
	}

	return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::SetDirectly);
}

void SGameMenuPageWidget::FadeIn()
{
	//Start the menu widget playing
	MenuWidgetAnimation.Play(this->AsShared());
	SetTitleAnimation(true);

	//Go into UI mode
	FSlateApplication::Get().SetKeyboardFocus(SharedThis(this));
}

FReply SGameMenuPageWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	//If we clicked anywhere, jump to the end
	if(MenuWidgetAnimation.IsPlaying())
	{
		MenuWidgetAnimation.JumpToEnd();
	}

	//Set the keyboard focus 
	return FReply::Handled()
		.SetUserFocus(SharedThis(this), EFocusCause::SetDirectly);
}

void SGameMenuPageWidget::ChangeOption(int32 InMoveBy)
{
	if (CurrentMenu.IsValid() == false)
	{
		return;
	}
	TSharedPtr<FGameMenuItem> MenuItem = CurrentMenu->GetItem(SelectedIndex);

	const int32 MinIndex = MenuItem->MinMultiChoiceIndex > -1 ? MenuItem->MinMultiChoiceIndex : 0;
	const int32 MaxIndex = MenuItem->MaxMultiChoiceIndex > -1 ? MenuItem->MaxMultiChoiceIndex : MenuItem->MultiChoice.Num()-1;
	const int32 CurIndex = MenuItem->SelectedMultiChoice;

	if (MenuItem->MenuItemType == EGameMenuItemType::MultiChoice)
	{
		if ( CurIndex + InMoveBy >= MinIndex && CurIndex + InMoveBy <= MaxIndex)
		{
			MenuItem->SelectedMultiChoice += InMoveBy;
			MenuItem->OnOptionChanged.ExecuteIfBound(MenuItem, MenuItem->SelectedMultiChoice);
			FSlateApplication::Get().PlaySound(MenuStyle->OptionChangeSound);
		}
		UpdateArrows(MenuItem);
	}
}

FReply SGameMenuPageWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Result = FReply::Unhandled();
	
	if ((CurrentMenu.IsValid() == true) && !bControlsLocked)
	{
		bool bNavigationLocked = PendingMainMenu.IsValid() || PendingSubMenu.IsValid();
		const EUINavigation NavDirection = FSlateApplication::Get().GetNavigationDirectionFromKey(InKeyEvent);
		const EUINavigationAction NavAction = FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent);

		if (bNavigationLocked == false)
		{
			if (NavDirection == EUINavigation::Up)
			{
				if (SelectedIndex > 0)
				{
					SelectionChanged(SelectedIndex - 1);
				}
				Result = FReply::Handled();
			}
			else if (NavDirection == EUINavigation::Down)
			{
				if (SelectedIndex + 1 < CurrentMenu->NumItems())
				{
					SelectionChanged(SelectedIndex + 1);
				}
				Result = FReply::Handled();
			}
			else if (NavDirection == EUINavigation::Left)
			{
				ChangeOption(-1);
				Result = FReply::Handled();
			}
			else if (NavDirection == EUINavigation::Right)
			{
				ChangeOption(1);
				Result = FReply::Handled();
			}
		}
		if (NavAction == EUINavigationAction::Accept)
		{
			ConfirmMenuItem();
			Result = FReply::Handled();
		} 
		else if (NavAction == EUINavigationAction::Back || InKeyEvent.GetKey() == EKeys::Gamepad_Special_Left)
		{
			MenuGoBack(true);
			Result = FReply::Handled();
		}
	}
	return Result;
}

FReply SGameMenuPageWidget::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (CurrentMenu.IsValid() == false)
	{
		return FReply::Unhandled();
	}

	//Focus the custom widget
	if (CurrentMenu.IsValid() && CurrentMenu->NumItems() == 1 && CurrentMenu->GetItem(0)->MenuItemType == EGameMenuItemType::CustomWidget)
	{
		return FReply::Handled().SetUserFocus(CurrentMenu->GetItem(0)->CustomWidget.ToSharedRef(),EFocusCause::SetDirectly);
	}

	return FReply::Handled().ReleaseMouseCapture().SetUserFocus(SharedThis(this));
}

void SGameMenuPageWidget::SetTitleAnimation(bool bShowTitle)
{
	if (TitleWidgetAnimation.IsPlaying() == false)
	{
		if (bShowTitle == true)
		{
			TitleWidgetAnimation.Play(this->AsShared());
		}
		else
		{
			TitleWidgetAnimation.PlayReverse(this->AsShared());
		}		
	}
	else
	{
		if (TitleWidgetAnimation.IsForward() == bShowTitle)
		{
			TitleWidgetAnimation.Reverse();
		}
		else
		{
			TitleWidgetAnimation.PlayReverse(this->AsShared());
		}
	}
}

void SGameMenuPageWidget::LockControls(bool bEnable)
{
	bControlsLocked = bEnable;
}

void SGameMenuPageWidget::OpenPendingSubMenu()
{
	if( (PendingSubMenu.IsValid() ) && ( MenuStyle->LayoutType == GameMenuLayoutType::SideBySide ) )
	{
		BuildPanelButtons(PendingSubMenu, SubPanel, INDEX_NONE);		
		SubMenuPanel.OpenPanel(this->AsShared());
		NextMenu = PendingSubMenu;
		NextMenu->MenuOpening();
		PendingSubMenu.Reset();
	}
	else
	{
		PendingSubMenu.Reset();
	}
}



void SGameMenuPageWidget::ResetMenu()
{
	while (MenuHistory.Num() != 0)
	{
		TSharedPtr< FGameMenuPage > MenuInfo = MenuHistory.Pop();
		MenuInfo->RemoveAllItems();
	}
	MainPanel->ClearChildren();
	if (SubPanel.IsValid())
	{
		SubPanel->ClearChildren();
	}
	PendingSubMenu.Reset();
	PendingMainMenu.Reset();
	CurrentMenu.Reset();
	SubPanel.Reset();	
}


TSharedPtr< FGameMenuPage > SGameMenuPageWidget::GetCurrentMenu()
{
	return CurrentMenu;
}
