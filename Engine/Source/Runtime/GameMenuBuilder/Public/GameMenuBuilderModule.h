// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * The public interface to this module
 */
class IGameMenuBuilderModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGameMenuBuilderModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IGameMenuBuilderModule >("GameMenuBuilder");
	}
};

class FGameMenuBuilderModule : public IGameMenuBuilderModule
{
	virtual void StartupModule() override
	{

	}
	void SetStyleName()
	{
	}

};
