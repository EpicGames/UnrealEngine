// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateStyle.h"

class ISlateStyle;

class FGameMenuBuilderStyle
{
public:

	static void Initialize(const FString StyleName);

	static void Shutdown();
	
	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set  */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();
	
private:

	static TSharedRef< class FSlateStyleSet > Create(const FString StyleName);

	static TSharedPtr< class FSlateStyleSet > SimpleStyleInstance;
};
