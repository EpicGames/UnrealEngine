// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class SGameMenuItemWidget;

namespace EGameMenuItemType
{
	enum Type
	{
		Root,
		Standard,
		MultiChoice,
		CustomWidget,
	};
};

class FGameMenuItem : public TSharedFromThis<FGameMenuItem>
{
public:
	/** confirm menu item delegate */
	DECLARE_DELEGATE(FOnConfirmMenuItem);

	/** multi-choice option changed, parameters are menu item itself and new multi-choice index  */
	DECLARE_DELEGATE_TwoParams(FOnOptionChanged, TSharedPtr<FGameMenuItem>, int32);

	/** delegate, which is executed by SSimpleMenuWidget if user confirms this menu item */
	FOnConfirmMenuItem OnConfirmMenuItem;

	/** multi-choice option changed, parameters are menu item itself and new multi-choice index */
	FOnOptionChanged OnOptionChanged;

	/** menu item type */
	EGameMenuItemType::Type MenuItemType;

	/** menu item text */
	FText Text;

	/** sub menu if present */
	TSharedPtr<class FGameMenuPage> SubMenu;

	/** shared pointer to actual slate widget representing the menu item */
	TSharedPtr<SGameMenuItemWidget> Widget;

	/** shared pointer to actual slate widget representing the custom menu item, ie whole options screen */
	TSharedPtr<SGameMenuItemWidget> CustomWidget;

	/** texts for multiple choice menu item (like INF AMMO ON/OFF or difficulty/resolution etc) */
	TArray<FText> MultiChoice;

	/** set to other value than -1 to limit the options range */
	int32 MinMultiChoiceIndex;

	/** set to other value than -1 to limit the options range */
	int32 MaxMultiChoiceIndex;

	/** selected multi-choice index for this menu item */
	int32 SelectedMultiChoice;

	/** true if this item is active */
	bool bInactive;

	/** constructor accepting menu item text */
	FGameMenuItem(FText _text)
	{
		Text = _text;
		MenuItemType = EGameMenuItemType::Standard;
		MinMultiChoiceIndex = MaxMultiChoiceIndex = SelectedMultiChoice = 0;
		bInactive = false;
	}

	/** custom widgets cannot contain sub menus, all functionality must be handled by custom widget itself */
	FGameMenuItem(TSharedPtr<SGameMenuItemWidget> InWidget)
	{
		MenuItemType = EGameMenuItemType::CustomWidget;
		CustomWidget = InWidget;
		MinMultiChoiceIndex = MaxMultiChoiceIndex = SelectedMultiChoice = 0;
		bInactive = false;
	}

	/** constructor for multi-choice item */
	FGameMenuItem(FText InText, TArray<FText> InOptions, int32 InDefaultIndex=0)
	{
		Text = InText;
		MenuItemType = EGameMenuItemType::MultiChoice;
		MultiChoice = InOptions;
		MinMultiChoiceIndex = MaxMultiChoiceIndex = -1;
		SelectedMultiChoice = InDefaultIndex;
		bInactive = false;
	}

	bool ConfirmPressed()
	{
		bool bHandled = false;
		if ( (bInactive == false ) && (OnConfirmMenuItem.IsBound() == true ) )
		{
			OnConfirmMenuItem.Execute();			
			bHandled = true;
		}
		return bHandled;
	}
	
	/** create special root item */
	static TSharedRef<FGameMenuItem> CreateRoot()
	{
		return MakeShareable(new FGameMenuItem());
	}
	
private:
	FGameMenuItem()
	{
		MenuItemType = EGameMenuItemType::Root;
	}
};
