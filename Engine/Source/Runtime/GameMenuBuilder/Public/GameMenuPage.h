// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "GameFramework/PlayerController.h"
#include "GameMenuItem.h"

class FString;
class SGameMenuItemWidget;
class SGameMenuPageWidget;
class UGameViewportClient;

class GAMEMENUBUILDER_API FGameMenuPage : public TSharedFromThis<FGameMenuPage>
{
public:
	FGameMenuPage();
	
	/** Delegate for when user is going back from submenu. */
	DECLARE_DELEGATE(FOnMenuGoBack);

	/** Delegate for when menu is hidden. */
	DECLARE_DELEGATE(FOnMenuHidden);
	
	/** Delegate for when menu is about to open. */
	DECLARE_DELEGATE(FOnMenuOpening);

	/** 
	 * Initialize the menu page widget and set this menu as root. 
	 * @param	InPCOwner		Player controller that owns this menu.
	 * @param	InStyleName		Name of the menu style to use.
	 * @param	InGameViewport	Gameviewport on which to place the menu.
	 * @returns true on success. (Will fail if viewport invalid)
	 */
	bool InitialiseRootMenu(APlayerController* InPCOwner, const FString& InStyleName, UGameViewportClient* InGameViewport);

	void DestroyRootMenu();

	/* Return the number of items in menu item list. */
	int32 NumItems() const;

	/* Get the specified item in the menu item list. Will assert if the value is out of range. */
	TSharedPtr<FGameMenuItem> GetItem(int32 Index) const;
	
	/* Is the supplied index with the valid range of menu items. Returns false if it is not. */
	bool IsValidMenuEntryIndex(int32 Index) const;
				
	/** Weak pointer to Owning player controller. */
	TWeakObjectPtr<APlayerController> PCOwner;

	/** Builds the menu widget and shows the menu. */
	void ShowRootMenu();

	/** Current selection in this menu. */
	int32 SelectedIndex;

	/** The menu title. */
	FText MenuTitle;

	/** 
	 * Add a menu item.
	 *
	 * @param	Text		The string for the item (EG START GAME)
	 * @param	InSubMenu	Any submenu associated with the item
	 * @returns A shared reference to the created item
	 */
	TSharedRef<FGameMenuItem> AddMenuItem(const FText& Text, TSharedPtr<FGameMenuPage> InSubMenu = nullptr);

	/*
	*
	* Add a menu item.
	*
	* @param	Text		Label of the menu item
	* @param	InObj		Menu page object
	* @param	InMethod	Method to call when selection changes.
	* @returns  SharedRef to the MenuItem that was created.
	*/
	template< class UserClass >
	inline TSharedRef<FGameMenuItem> AddMenuItem(const FText& InText, UserClass* InObj, typename FGameMenuItem::FOnConfirmMenuItem::TMethodPtr< UserClass > InMethod)
	{
		TSharedPtr<FGameMenuItem> Item = MakeShareable(new FGameMenuItem(InText));
		Item->OnConfirmMenuItem.BindSP(InObj, InMethod);
		MenuItems.Add(Item);
		return Item.ToSharedRef();
	}

	/**
	 * Add a menu entry with a variable number of selectable options
	 *
	 * @param	Text		Label of the menu item
	 * @param	OptionsList	List of selectable items
	 * @param	InObj		Menu page object
	 * @param	InMethod	Method to call when selection changes.
	 * @returns  SharedRef to the MenuItem that was created.
	 */
	template< class UserClass >
	inline TSharedRef<FGameMenuItem> AddMenuItemWithOptions(const FText& Text, const TArray<FText>& OptionsList, UserClass* InObj, typename FGameMenuItem::FOnOptionChanged::TMethodPtr< UserClass > InMethod)
	{
		TSharedPtr<FGameMenuItem> Item = MakeShareable(new FGameMenuItem(Text, OptionsList));
		Item->OnOptionChanged.BindSP(InObj, InMethod);
		MenuItems.Add(Item);
		return Item.ToSharedRef();
	}

	/**
	 * Add a custom menu entry 
	 *
	 * @param	Text		Label of the menu item
	 * @param	
	 * @param	
	 * @param	
	 * @returns  SharedRef to the MenuItem that was created.
	 */
	template< class UserClass >
	inline TSharedRef<FGameMenuItem> AddCustomMenuItem(const FText& Text, TSharedPtr<SGameMenuItemWidget> CustomWidget, UserClass* InObj, typename FGameMenuItem::FOnOptionChanged::TMethodPtr< UserClass > InMethod)
	{
		TSharedPtr<FGameMenuItem> Item = MakeShareable(new FGameMenuItem(CustomWidget));
		Item->OnOptionChanged.BindSP(InObj, InMethod);
		MenuItems.Add(Item);
		return Item.ToSharedRef();
	}

	/**
	* Add a handler for the menu being canceled.
	*
	* @param	InObj		Menu page object
	* @param	InMethod	Method to call when selection changes.
	*/
	template< class UserClass >
	inline void SetCancelHandler(UserClass* InObj, typename FGameMenuPage::FOnMenuGoBack::TMethodPtr< UserClass > InMethod)
	{
		OnGoBackCancel.BindSP(InObj, InMethod);
	}
	
	/**
	* Add a handler for the menu being canceled.
	*
	* @param	InObj		Menu page object
	* @param	InMethod	Method to call when menu has been hidden.
	*/
	template< class UserClass >
	inline void SetOnHiddenHandler(UserClass* InObj, typename FGameMenuPage::FOnMenuHidden::TMethodPtr< UserClass > InMethod)
	{
		OnMenuHidden.BindSP(InObj, InMethod);
	}

	/**
	* Add a handler for the menu being accepted.
	*
	* @param	InObj		Menu page object
	* @param	InMethod	Method to call when selection changes.
	*/
	template< class UserClass >
	inline void SetAcceptHandler(UserClass* InObj, typename FGameMenuPage::FOnMenuGoBack::TMethodPtr< UserClass > InMethod)
	{
		OnGoBack.BindSP(InObj, InMethod);
	}

	/**
	* Add a handler for the menu being opened.
	*
	* @param	InObj		Menu page object
	* @param	InMethod	Method to call when menu is about to open.
	*/
	template< class UserClass >
	inline void SetOnOpenHandler(UserClass* InObj, typename FGameMenuPage::FOnMenuHidden::TMethodPtr< UserClass > InMethod)
	{
		OnMenuOpening.BindSP(InObj, InMethod);
	}

	/**
	 * Lock/Unlock the menu and prevent user interaction.
	 *
	 * @param		bLockState	true to lock
	 */
	void LockControls(bool bLockState);
		
	/** The widget that is the menu. */
	TSharedPtr<SGameMenuPageWidget> RootMenuPageWidget;

	/** Executed when user want's to go back to previous menu. */
	void MenuGoBack()
	{
		OnGoBack.ExecuteIfBound();
	}

	/** Called when user want's to CANCEL and go back to previous menu. */
	void MenuGoBackCancel()
	{
		OnGoBackCancel.ExecuteIfBound();
	}

	/* Called when the menu has finished hiding. */
	void MenuHidden()
	{
		OnMenuHidden.ExecuteIfBound();
	}

	/* Called when the menu is about to be opened. */
	void MenuOpening()
	{
		OnMenuOpening.ExecuteIfBound();
	}

	/* Hide the menu. */
	void HideMenu();

	/* Remove all the items from the item array. */
	void RemoveAllItems();

protected:	
	
	/** Executed when user want's to go back to previous menu. */
	FOnMenuGoBack OnGoBack;

	/** Executed when user want's to CANCEL and go back to previous menu. */
	FOnMenuGoBack OnGoBackCancel;

	/** Delegate, which is executed when menu is finished hiding. */
	FOnMenuHidden OnMenuHidden;

	/** Delegate, which is executed when menu is about to open. */
	FOnMenuOpening OnMenuOpening;

	/* Array of menu items that represents the menu. */
	typedef TArray< TSharedPtr<FGameMenuItem> > MenuItemList;
	MenuItemList	MenuItems;
};

