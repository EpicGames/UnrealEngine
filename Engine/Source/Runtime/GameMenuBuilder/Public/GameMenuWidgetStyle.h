// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "GameMenuWidgetStyle.generated.h"

UENUM()
namespace GameMenuLayoutType
{
	enum Type : int
	{
		/* Single one page menu. */
		Single,
		/* Side by side menu. Main menu on left, any sub menu on right. */
		SideBySide
	};
}

USTRUCT()
struct FGameMenuStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FGameMenuStyle();
	virtual ~FGameMenuStyle();

	// FSlateWidgetStyle
	virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static const FGameMenuStyle& GetDefault();

	/**
	 * The brush used to draw the top of the menu
	 */
	UPROPERTY(EditAnywhere, Category = Brushes)
	FSlateBrush MenuTopBrush;
	FGameMenuStyle& SetMenuTopBrush(const FSlateBrush& InMenuTopBrush) { MenuTopBrush = InMenuTopBrush; return *this; }

	/**
	 * The brush used to draw the menu border
	 */
	UPROPERTY(EditAnywhere, Category = Brushes)
	FSlateBrush MenuFrameBrush;
	FGameMenuStyle& SetMenuFrameBrush(const FSlateBrush& InMenuFrameBrush) { MenuFrameBrush = InMenuFrameBrush; return *this; }

	/**
	* The brush used to draw the menu background
	*/
	UPROPERTY(EditAnywhere, Category = Brushes)
	FSlateBrush MenuBackgroundBrush;
	FGameMenuStyle& SetMenuBackgroundBrush(const FSlateBrush& InMenuBackgroundBrush) { MenuBackgroundBrush = InMenuBackgroundBrush; return *this; }

	/**
	 * The brush used to draw the selected menu item
	 */
	UPROPERTY(EditAnywhere, Category = Brushes)
	FSlateBrush MenuSelectBrush;
	FGameMenuStyle& SetMenuSelectBrush(const FSlateBrush& InMenuSelectBrush) { MenuSelectBrush = InMenuSelectBrush; return *this; }

	/**
	 * The sound the menu should play when entering a sub-menu
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound MenuEnterSound;
	FGameMenuStyle& SetMenuEnterSound(const FSlateSound& InMenuEnterSound) { MenuEnterSound = InMenuEnterSound; return *this; }

	/**
	 * The sound the menu should play when returning to a parent menu
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound MenuBackSound;
	FGameMenuStyle& SetMenuBackSound(const FSlateSound& InMenuBackSound) { MenuBackSound = InMenuBackSound; return *this; }

	/**
	 * The sound the menu should play when a menu option is changed
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound OptionChangeSound;
	FGameMenuStyle& SetOptionChangeSound(const FSlateSound& InOptionChangeSound) { OptionChangeSound = InOptionChangeSound; return *this; }

	/**
	 * The sound the menu should play when the selected menu item is changed
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound MenuItemChangeSound;
	FGameMenuStyle& SetMenuItemChangeSound(const FSlateSound& InMenuItemChangeSound) { MenuItemChangeSound = InMenuItemChangeSound; return *this; }

	/**
	 * The sound the menu should play when the selected menu item is disabled or unbound
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound MenuItemInactiveSound;
	FGameMenuStyle& SetMenuItemInactiveSound(const FSlateSound& InMenuItemInactiveSound) { MenuItemInactiveSound = InMenuItemInactiveSound; return *this; }

	/**
	 * The sound the menu should play when the selected menu item is chosen
	 */
	UPROPERTY(EditAnywhere, Category = Sound)
	FSlateSound MenuItemChosenSound;
	FGameMenuStyle& SetMenuItemChosenSound(const FSlateSound& InMenuItemChosenSound) { MenuItemChosenSound = InMenuItemChosenSound; return *this; }
	
	/**
	 * The left hand margin for the main menu 
	 */
	UPROPERTY(EditAnywhere, Category = Layout)
	float LeftMarginPercent;
	FGameMenuStyle& SetMenuLeftMarginPercent(const float InLeftMarginPercent) { LeftMarginPercent = InLeftMarginPercent; return *this; }
	
	/**
	* The left hand margin for the sub menu
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	float SubMenuMarginPercent;
	FGameMenuStyle& SetSubMenuMarginPercent(const float InSubMenuMarginPercent) { SubMenuMarginPercent = InSubMenuMarginPercent; return *this; }
	
	/**
	* Layout type.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	TEnumAsByte<GameMenuLayoutType::Type> LayoutType;
	FGameMenuStyle& SetMenuTitleColor(const GameMenuLayoutType::Type InLayoutType) { LayoutType = InLayoutType; return *this; }

	/**
	* Title border margin.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	FMargin TitleBorderMargin;
	FGameMenuStyle& SetTitleBorderMargin(const FMargin InTitleBorderMargin) { TitleBorderMargin = InTitleBorderMargin; return *this; }

	/**
	* Item border margin.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	FMargin ItemBorderMargin;
	FGameMenuStyle& SetItemBorderMargin(const FMargin InItemBorderMargin) { ItemBorderMargin = InItemBorderMargin; return *this; }

	/**
	* Title horizontal alignment.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	TEnumAsByte<EHorizontalAlignment> TitleHorizontalAlignment;
	FGameMenuStyle& SetTitleHorizontalAlignment(const EHorizontalAlignment InTitleHorizontalAlignment) { TitleHorizontalAlignment = InTitleHorizontalAlignment; return *this; }

	/**
	* Title vertical alignment.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	TEnumAsByte<EVerticalAlignment> TitleVerticalAlignment;
	FGameMenuStyle& SetTitleVerticalAlignment(const EVerticalAlignment InTitleVerticalAlignment) { TitleVerticalAlignment = InTitleVerticalAlignment; return *this; }

	/**
	* Panel vertical alignment.
	*/
	UPROPERTY(EditAnywhere, Category = Layout)
	TEnumAsByte<EVerticalAlignment> PanelsVerticalAlignment;
	FGameMenuStyle& SetPanelsVerticalAlignment(const EVerticalAlignment InPanelsVerticalAlignment) { PanelsVerticalAlignment = InPanelsVerticalAlignment; return *this; }

	/**
	 * Speed at which the menu initially appears. 
	 */
	UPROPERTY(EditAnywhere, Category = Animation)
	float AnimationSpeed;
	FGameMenuStyle& SetMainAnimDuration(const float InAnimationSpeed) { AnimationSpeed = InAnimationSpeed; return *this; }
	
	/**
	* Color of the text.
	*/
	UPROPERTY(EditAnywhere, Category = Color)
	FSlateColor TextColor;
	FGameMenuStyle& SetTextColor(const FSlateColor InTextColor) { TextColor = InTextColor; return *this; }
};


/**
*/
UCLASS(hidecategories = Object, MinimalAPI)
class UGameMenuWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_UCLASS_BODY()

public:
	/** The actual data describing the menu's appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FGameMenuStyle MenuStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast< const struct FSlateWidgetStyle* >(&MenuStyle);
	}
};
