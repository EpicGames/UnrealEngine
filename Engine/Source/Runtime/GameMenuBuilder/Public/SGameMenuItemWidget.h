// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/PlayerController.h"
#include "GameMenuWidgetStyle.h"

struct FGeometry;
struct FPointerEvent;

// Menu item widget
class GAMEMENUBUILDER_API SGameMenuItemWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnArrowPressed, int );

	SLATE_BEGIN_ARGS(SGameMenuItemWidget)
	{}

	/** style to use for this menu item */
	SLATE_STYLE_ARGUMENT(FGameMenuStyle, MenuStyle)

	/** weak pointer to the parent PC */
	SLATE_ARGUMENT(TWeakObjectPtr<APlayerController>, PCOwner)

	/** called when the button is clicked */
	SLATE_EVENT(FOnClicked, OnClicked)

	/** called when the left or right arrow is clicked */
	SLATE_EVENT(FOnArrowPressed, OnArrowPressed)

	/** menu item text attribute */
	SLATE_ATTRIBUTE(FText, Text)

	/** is it multi-choice item? */
	SLATE_ARGUMENT(bool, bIsMultichoice)

	/** menu item option text attribute */
	SLATE_ATTRIBUTE(FText, OptionText)

	/** menu item text transparency when item is not active, optional argument */
	SLATE_ARGUMENT(TOptional<float>, InactiveTextAlpha)

	/** end of slate attributes definition */
	SLATE_END_ARGS()

	/** needed for every widget */
	void Construct(const FArguments& InArgs);

	//~ Begin SWidget Interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget Interface

	/** Sets this menu item as active (selected) */
	void SetMenuItemActive(bool bIsMenuItemActive);

	/** set in option item to enable left arrow*/
	EVisibility LeftArrowVisible;

	/** set in option item to enable right arrow*/
	EVisibility RightArrowVisible;

	/** Set pointer to our parent Player Controller. */
	void SetMenuOwner(TWeakObjectPtr<class APlayerController> InPCOwner);
	
	/** Set pointer to our style. */
	void SetMenuStyle(const FGameMenuStyle* InMenuStyle);

	void SetClickedDelegate(FOnClicked InOnClicked);

	/** Delegate to execute when one of arrows was pressed. */
	void SetArrowPressedDelegate(FOnArrowPressed InOnArrowPressed);

protected:
	/** Delegate to execute when the button is clicked. */
	FOnClicked OnClicked;

	/** Delegate to execute when one of arrows was pressed. */
	FOnArrowPressed OnArrowPressed;

	/** Menu item text attribute. */
	TAttribute< FText > Text;

	/** Menu item option text attribute. */
	TAttribute< FText > OptionText;

	/** Inactive text alpha value. */
	float InactiveTextAlpha;

	/** Active item flag. */
	bool bIsActiveMenuItem;

	/** Does this menu item represent multi-choice field. */
	bool bIsMultichoice;

	/** Pointer to our parent Player Controller. */
	TWeakObjectPtr<class APlayerController> PCOwner;

	/* The Style of the menu. */
	const FGameMenuStyle* MenuStyle;

private:
	
	/** Getter for menu item background color. */
	FSlateColor GetButtonBgColor() const;

	/** Getter for menu item text color. */
	FSlateColor GetButtonTextColor() const;

	/** Getter for left option arrow visibility. */
	EVisibility GetLeftArrowVisibility() const;

	/** Getter for right option arrow visibility. */
	EVisibility GetRightArrowVisibility() const;

	/** Right arrrow pressed delegate. */
	FReply OnRightArrowDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Left Arrow pressed handler. */
	FReply OnLeftArrowDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

};


