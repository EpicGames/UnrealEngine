// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/IntPoint.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/PlayerController.h"
#include "GameMenuWidgetStyle.h"

class UGameViewportClient;
class FGameMenuPage;
class SVerticalBox;
class SWidget;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

namespace EPanelState
{
	enum Type
	{
		/* Menu is opening. */
		Opening,
		/* Menu is open. */
		Open,
		/* Menu is closing. */
		Closing,
		/* Menu is closed. */
		Closed
	};

};

DECLARE_DELEGATE_OneParam(FPanelStateChanged, bool);

// Simple class to contain the menu panels/animations
class FMenuPanel
{
public:
	FMenuPanel();

	void Tick(float Delta);

	/* Close the panel */
	void ClosePanel(TSharedRef<SWidget> OwnerWidget);
	
	/* Open the panel */
	void OpenPanel(TSharedRef<SWidget> OwnerWidget);

	/* Force the panel to be fully open. */
	void ForcePanelOpen();

	/* Force the panel to be fully closed. */
	void ForcePanelClosed();
	
	void SetStyle(const FGameMenuStyle* InStyle);

	/* Delegate called when the panel becomes open or closed. */
	FPanelStateChanged	OnStateChanged;
	
	/* Animation curve/handle for panel animation. */
	FCurveHandle AnimationHandle;

	/* The current state of the panel. */
	EPanelState::Type	CurrentState;

private:		
	/* Animation sequence used to open or close the panel. */
	FCurveSequence AnimationSequence;
};

class FGameMenuPage;

// Simple Menu page widget class
class GAMEMENUBUILDER_API SGameMenuPageWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameMenuPageWidget)
	: _PCOwner() {}

	/** style to use for this menu item. */
	SLATE_STYLE_ARGUMENT(FGameMenuStyle, MenuStyle)

	/** weak pointer to the parent HUD base. */
	SLATE_ARGUMENT(TWeakObjectPtr<APlayerController>, PCOwner)

	/** is this main menu or in game menu ? */
	SLATE_ARGUMENT(bool, GameMenu)

	/** always goes here */
	SLATE_END_ARGS()
		
	/** Delegate to call when in-game menu should be hidden using controller buttons - 
	it's workaround as when joystick is captured, even when sending FReply::Unhandled, binding does not receive input :( */
	DECLARE_DELEGATE(FOnToggleMenu);
	
	/** Delegate for selection changing. Passes old and new selection. */
	DECLARE_DELEGATE_TwoParams(FOnSelectionChanged, TSharedPtr<class FGameMenuItem>, TSharedPtr<class FGameMenuItem>);

	/** every widget needs a construction function. */
	void Construct(const FArguments& InArgs);

	//~ Begin SWidget Interface
	/** Update function. */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Mouse button down handler. */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Key down handler. */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** says that we can support keyboard focus. */
	virtual bool SupportsKeyboardFocus() const override { return true; }

	/** The menu sets up the appropriate mouse settings upon focus. */
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

	//~ End SWidget Interface

	/**
	 * Callback handler for when the state of the main panel changes.
	 *
	 * @param bWasOpened	whether the panel was opened or closed. is TRUE if panel opened.
	 *
	 */ 
	virtual void OnMainPanelStateChange(bool bWasOpened);

	
	/**
 	 * Callback handler for when the state of the sub panel changes.b 
	 *
	 * @param bWasOpened	whether the panel was opened or closed. is TRUE if panel opened.
	 *
	 */ 
	virtual void OnSubPanelStateChange(bool bWasOpened);

	/** Setups misc animation sequences. */
	void SetupAnimations();

	/** Make the currently selected menu sub menu new main menu if valid. */
	void EnterSubMenu(TSharedPtr<FGameMenuPage> InSubMenu);

	/**
	 * Go back the the previous menu.
	 * 
	 * @param	bIsCancel	if true will be treated as CANCEL (IE escape pressed).
	 *
	 */
	void MenuGoBack(bool bIsCancel=false);

	/** Confirms current menu item and performs an action. Will also play selection sound. */
	void ConfirmMenuItem();

	/**
	 * Show the given menu and make it the current menu. 
	 *
	 * @param InMenu	The menu to show and set as the current menu. If this is NULL Current menu will be used if valid.
	 *
	 */
	void BuildAndShowMenu(TSharedPtr< FGameMenuPage > InMenu);

	/** Hide the menu. */
	void HideMenu();

	/**
	 * Updates arrows visibility for multi-choice menu item.
	 *
	 * @param	InMenuItem The item to update.
	 *
	 */
	void UpdateArrows(TSharedPtr<class FGameMenuItem> InMenuItem);

	/**
	 * Change the currently selection option of the currently selected menu item.
	 *
	 * @param	InMoveBy		Number to change the option by.
	 *
	 */
	void ChangeOption(int32 InMoveBy);

	/** disable/enable moving around menu. */
	void LockControls(bool bEnable);

	/** 
	 * Set the current menu and 'open' it as the main panel. 
	 *
	 * @param InMenu	The menu to open and set as current.
	 *
	 */
	void OpenMainPanel(TSharedPtr< FGameMenuPage > InMenu);
		
	/**
	* Select a given item from an index
	*
	* @param InSelection	The index of the item to select
	* @returns true if selection changed.
	*/
	bool SelectItem(int32 InSelection);

	void ResetMenu();

	/* The viewport I am attached to. */
	TWeakObjectPtr<UGameViewportClient> MyGameViewport;

	TSharedPtr< FGameMenuPage > GetCurrentMenu();
protected:

	/* Set the current menu. Also sets the owner widget of that menu to this and resets the previous menu owner. */
	void SetCurrentMenu(TSharedPtr< FGameMenuPage > InMenu);

	/** Sets hit test invisibility when console is up. */
	EVisibility GetSlateVisibility() const;

	/* Returns true if the submenu should be visible. */
	EVisibility GetSubMenuVisiblity() const;

	/** Gets current scale for drawing menu. */
	float GetUIScale() const;

	/** 
	 * Rebuild the widgets in the main menu widget.
	 * Selected item will be set from InPreviousIndex unless it is invalid.
	 *
	 * @param		InPanel		The FGameMenuPage we are building the panel from.
	 * @param		InBox		The SVerticalBox in which to build the widgets from the given panel.
	 * @param		PreviousIndex	The previous menu index. (Can be INDEX_NONE)
	 *
	 */
	void BuildPanelButtons(TSharedPtr< FGameMenuPage > InPanel, TSharedPtr<SVerticalBox> InBox, int32 InPreviousIndex);

	/** Misc Getters used for animating/fading various menu facets. */
	FLinearColor GetPanelsColor() const;	
	FSlateColor GetPanelsBackgroundColor() const;
	FMargin GetMainMenuOffset() const;
	FMargin GetSubMenuOffset() const;
	FMargin GetMenuTitleOffset() const;
	FMargin GetMenuOffset() const;
	FMargin GetMenuItemPadding() const;
	FMargin GetSubMenuItemPadding() const;
	FSlateColor GetTitleColor() const;
	FSlateColor GetTextColor() const;

	/** 
	 * Callback for when one of the menu items is selected. 
	 *
	 * @param	SelectionIndex	The index selection.
	 * @returns	Will return UNHANDLED if the current menu is invalid, HANDLED in all other cases including if the controls are locked.
	 */
	virtual FReply SelectionChanged(int32 SelectionIndex);

	/** 
	 * Callback for handling mouse click. 
	 *
	 * @param	SelectionIndex	The index selection.
	 * @returns	Will return UNHANDLED if the current menu is invalid, HANDLED in all other cases including if the controls are locked.
	 */	
	virtual FReply MouseButtonClicked(int32 SelectionIndex);

	/** 
	 * Gets currently selected multi-choice option. 
	 *
	 * @param	SelectionIndex	The index selection.
	 * @returns	Will return UNHANDLED if the current menu is invalid, HANDLED in all other cases including if the controls are locked.
	 */
	FText GetOptionText(TSharedPtr<class FGameMenuItem> MenuItem) const;

	/** Gets current menu title string. */
	FText GetMenuTitle() const; 

	/** Gets the menu title visibility based on if the title text is empty. */
	EVisibility GetMenuTitleVisibility() const;

	/** Start the main menu open fade/anim and set keyboard focu.s */
	void FadeIn();

	/** 
	 * Animates the title widget.
	 *
	 * @param	bShowTitle	true to show the title, false to hide it.
	 */
	void SetTitleAnimation(bool bShowTitle);

	/* Opens any pending sub menu if there is one. */
	void OpenPendingSubMenu();

	/** Control for hiding the menu - Defaults to start button. */
	//FKey ControllerHideMenuKey;
	
	/** Current menu title if present. */
	FText CurrentMenuTitle;

	/** If console is currently opened. */
	bool bConsoleVisible;
	
	/** Container instance for main panel. */
	FMenuPanel	MainMenuPanel;

	/** Container instance for sub menu panel (only relevant with side by side layout). */
	FMenuPanel	SubMenuPanel;
	
	/** Next menu (for transition and displaying as the right menu). */
	TSharedPtr< FGameMenuPage > NextMenu;

	/** Currently active menu. */
	TSharedPtr< FGameMenuPage > CurrentMenu;

	/** Current UI scale attribute. */
	TAttribute<float> UIScale;

	/** Our curve sequence and the related handles. */
	FCurveSequence MenuWidgetAnimation;

	/** Used for main menu logo fade in animation at the beginning.  */
	FCurveHandle TopColorCurve;

	/** Used for menu background fade in animation at the beginning. */
	FCurveHandle ColorFadeCurve;

	/** Used for menu buttons slide in animation at the beginning. */
	FCurveHandle MenuAnimationCurve;

	/** Our curve sequence and the related handles. */
	FCurveSequence TitleWidgetAnimation;

	/* Used to animate the title widget (NYI) */
	FCurveHandle TitleWidgetCurve;

	/** Weak pointer to our parent Player Controller. */
	TWeakObjectPtr<class APlayerController> PCOwner;

	/** Screen resolution. */
	FIntPoint ScreenRes;

	/** Animation type index. */
	int32 MainAnimNumber;

	/** Selected index of current menu. */
	int32 SelectedIndex;
	
	/** Flag when playing hiding animation. */
	uint8 bMenuHiding : 1;

	/** Flag when playing hiding animation. */
	uint8 bMenuHidden : 1;

 	/** If this is in game menu. */
 	uint8 bGameMenu : 1;

	/** If moving around menu is currently locked. */
	uint8 bControlsLocked : 1;
	
	/** Menu that will override current one after transition animation. */
	TSharedPtr< FGameMenuPage > PendingMainMenu;
	
	/** Menu that will become the submenu after transition animation. */
	TSharedPtr< FGameMenuPage > PendingSubMenu;
	
	/** Current menu layout box. */
	TSharedPtr<SVerticalBox> MainPanel;

	/** Sub menu layout box. */
	TSharedPtr<SVerticalBox> SubPanel;

	/** Style to use for this menu item. */
	const FGameMenuStyle* MenuStyle;
		
	/** Stack of previous menus. */
	TArray< TSharedPtr< FGameMenuPage>> MenuHistory;

	/** Bind if menu should be hidden from outside by controller button. */
	FOnToggleMenu OnToggleMenu;
	
	/** Executed when the user selects an item. */
	FOnSelectionChanged OnSelectionChange;
};
